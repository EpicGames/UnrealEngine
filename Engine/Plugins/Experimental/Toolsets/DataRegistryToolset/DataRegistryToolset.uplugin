{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "DataRegistryToolset",
	"Description": "Toolset for querying and inspecting Data Registries",
	"Category": "Other",
	"EditorOnly": true,
	"CreatedBy": "Epic Games",
	"CreatedByURL": "",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"NoRedist": true,
	"EnabledByDefault": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "DataRegistryToolset",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit",
			"TargetAllowList": ["Editor"]
		}
	],
	"Plugins": [
		{
			"Name": "ToolsetRegistry",
			"Enabled": true
		},
		{
			"Name": "DataRegistry",
			"Enabled": true
		}
	]
}
