// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DataRegistryToolset : ModuleRules
{
	public DataRegistryToolset(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

		PublicDependencyModuleNames.Add("Core");

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"CoreUObject",
			"DataRegistry",
			"Engine",
			"GameplayTags",
			"Json",
			"ToolsetRegistry",
		});
	}
}
