// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTools.h"

#include "DataRegistry.h"
#include "DataRegistryId.h"
#include "DataRegistrySource.h"
#include "DataRegistrySubsystem.h"
#include "GameplayTagContainer.h"
#include "Kismet/KismetSystemLibrary.h"
#include "ToolsetRegistry/ToolsetLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DataRegistryTools)

namespace
{
	const UDataRegistrySubsystem& GetSubsystemChecked()
	{
		const UDataRegistrySubsystem* Subsystem = UDataRegistrySubsystem::Get();
		checkf(Subsystem, TEXT("DataRegistrySubsystem is not available."));
		return *Subsystem;
	}

	/** Finds a registry by name. Returns null and raises a script error if not found. */
	const UDataRegistry* FindRegistryOrRaise(const FString& RegistryName)
	{
		const UDataRegistrySubsystem& Subsystem = GetSubsystemChecked();
		if (const UDataRegistry* Registry = Subsystem.GetRegistryForType(FName(*RegistryName)))
		{
			return Registry;
		}

		TArray<UDataRegistry*> AllRegistries;
		Subsystem.GetAllRegistries(AllRegistries);

		TArray<FString> AvailableNames;
		AvailableNames.Reserve(AllRegistries.Num());
		for (const UDataRegistry* Candidate : AllRegistries)
		{
			if (Candidate != nullptr)
			{
				AvailableNames.Add(Candidate->GetRegistryType().ToString());
			}
		}

		const FString Available = AvailableNames.IsEmpty()
			? TEXT("(none)") : FString::Join(AvailableNames, TEXT(", "));
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Registry '%s' not found. Available: %s"), *RegistryName, *Available));
		return nullptr;
	}

	/** Populates the portion of FDataRegistryInfo shared by ListRegistries and GetRegistryInfo. */
	FDataRegistryInfo BuildRegistryInfo(const UDataRegistry& Registry)
	{
		FDataRegistryInfo Info;
		Info.RegistryName = Registry.GetRegistryType().ToString();
		Info.ItemStruct = Registry.GetItemStruct();

		TArray<FDataRegistryId> Ids;
		Registry.GetPossibleRegistryIds(Ids);
		Info.ItemCount = Ids.Num();
		Info.Availability = Registry.GetLowestAvailability();
		return Info;
	}

	/** Builds a source summary for one UDataRegistrySource* (null-safe). */
	FDataRegistrySourceSummary BuildSourceSummary(UDataRegistrySource* Source)
	{
		FDataRegistrySourceSummary Summary;
		if (Source == nullptr)
		{
			return Summary;
		}

		Summary.SourceClass = Source->GetClass();
		Summary.DebugString = Source->GetDebugString();
		Summary.SourceAssetPath = Source->GetSourceAssetPath();
		Summary.Availability = Source->GetSourceAvailability();
		Summary.bIsInitialized = Source->IsInitialized();
		Summary.bIsTransient = Source->IsTransientSource();

		if (const UDataRegistrySource* Original = Source->GetOriginalSource())
		{
			if (Original != Source)
			{
				Summary.ParentSourceDebugString = Original->GetDebugString();
			}
		}
		return Summary;
	}

	TArray<FDataRegistrySourceSummary> BuildSourceSummaries(
		const TArray<TObjectPtr<UDataRegistrySource>>& Sources)
	{
		TArray<FDataRegistrySourceSummary> Result;
		Result.Reserve(Sources.Num());
		for (const TObjectPtr<UDataRegistrySource>& Source : Sources)
		{
			Result.Add(BuildSourceSummary(Source.Get()));
		}
		return Result;
	}
}

TArray<FString> UDataRegistryTools::ListRegistries(const UScriptStruct* StructFilter)
{
	TArray<UDataRegistry*> AllRegistries;
	GetSubsystemChecked().GetAllRegistries(AllRegistries);

	TArray<FString> Results;
	Results.Reserve(AllRegistries.Num());

	for (const UDataRegistry* Registry : AllRegistries)
	{
		if (Registry == nullptr)
		{
			continue;
		}
		if (StructFilter != nullptr)
		{
			const UScriptStruct* ItemStruct = Registry->GetItemStruct();
			if (ItemStruct == nullptr || !ItemStruct->IsChildOf(StructFilter))
			{
				continue;
			}
		}
		Results.Add(Registry->GetRegistryType().ToString());
	}

	return Results;
}

FDataRegistryInfo UDataRegistryTools::GetRegistryInfo(const FString& RegistryName)
{
	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return FDataRegistryInfo();
	}

	FDataRegistryInfo Info = BuildRegistryInfo(*Registry);
	Info.Description = Registry->GetRegistryDescription().ToString();

	const FDataRegistryIdFormat& IdFormat = Registry->GetIdFormat();
	if (IdFormat.BaseGameplayTag.IsValid())
	{
		Info.IdFormat = IdFormat.BaseGameplayTag.GetTagName().ToString();
	}
	return Info;
}

FString UDataRegistryTools::GetSchema(const FString& RegistryName)
{
	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return FString();
	}

	const UScriptStruct* ItemStruct = Registry->GetItemStruct();
	if (ItemStruct == nullptr)
	{
		UKismetSystemLibrary::RaiseScriptError(FString::Printf(
			TEXT("Registry '%s' has no item struct."), *RegistryName));
		return FString();
	}

	return UToolsetLibrary::ListStructProperties(ItemStruct, false);
}

TArray<FString> UDataRegistryTools::ListItems(const FString& RegistryName)
{
	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return TArray<FString>();
	}

	TArray<FDataRegistryId> AllIds;
	Registry->GetPossibleRegistryIds(AllIds);

	TArray<FString> Result;
	Result.Reserve(AllIds.Num());
	for (const FDataRegistryId& Id : AllIds)
	{
		Result.Add(Id.ItemName.ToString());
	}
	return Result;
}

TArray<FDataRegistrySourceSummary> UDataRegistryTools::ListDataSources(
	const FString& RegistryName)
{
	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return TArray<FDataRegistrySourceSummary>();
	}
	return BuildSourceSummaries(Registry->GetDataSources());
}

TArray<FDataRegistrySourceSummary> UDataRegistryTools::ListRuntimeSources(
	const FString& RegistryName)
{
	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return TArray<FDataRegistrySourceSummary>();
	}
	return BuildSourceSummaries(Registry->GetRuntimeSources());
}

TMap<FString, FInstancedStruct> UDataRegistryTools::GetItems(
	const FString& RegistryName, const TArray<FString>& ItemNames)
{
	TMap<FString, FInstancedStruct> Result;

	const UDataRegistry* Registry = FindRegistryOrRaise(RegistryName);
	if (Registry == nullptr)
	{
		return Result;
	}

	const UDataRegistrySubsystem& Subsystem = GetSubsystemChecked();
	const FName RegistryTypeName = Registry->GetRegistryType();

	for (const FString& Name : ItemNames)
	{
		const FDataRegistryId ItemId(RegistryTypeName, FName(*Name));
		const uint8* ItemMemory = nullptr;
		const UScriptStruct* ItemStruct = nullptr;
		const FDataRegistryCacheGetResult CacheResult =
			Subsystem.GetCachedItemRaw(ItemMemory, ItemStruct, ItemId);

		if (!CacheResult.WasFound() || ItemMemory == nullptr || ItemStruct == nullptr)
		{
			continue;
		}

		FInstancedStruct Instance;
		Instance.InitializeAs(ItemStruct, ItemMemory);
		Result.Add(Name, MoveTemp(Instance));
	}

	return Result;
}
