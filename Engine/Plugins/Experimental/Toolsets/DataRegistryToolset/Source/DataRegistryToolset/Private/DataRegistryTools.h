// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "DataRegistryTypes.h"
#include "StructUtils/InstancedStruct.h"
#include "ToolsetRegistry/ToolsetDefinition.h"
#include "UObject/SoftObjectPath.h"

#include "DataRegistryTools.generated.h"

class UClass;
class UScriptStruct;

/// Detailed information about a single Data Registry.
USTRUCT(BlueprintType)
struct FDataRegistryInfo
{
	GENERATED_BODY()

	/// The registry name.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FString RegistryName;

	/// Human-readable description.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FString Description;

	/// The UScriptStruct used for items in this registry. May be null.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	TObjectPtr<const UScriptStruct> ItemStruct;

	/// Number of item IDs known to this registry.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	int32 ItemCount = 0;

	/// Lowest availability across all sources on this registry. DoesNotExist indicates the
	/// registry has not been initialized.
	UPROPERTY(VisibleAnywhere, Category = "DataRegistry")
	EDataRegistryAvailability Availability = EDataRegistryAvailability::DoesNotExist;

	/// Base GameplayTag of the ID format, if any.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FString IdFormat;
};

/// Summary information about a single UDataRegistrySource entry.
USTRUCT(BlueprintType)
struct FDataRegistrySourceSummary
{
	GENERATED_BODY()

	/// Class of the source.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	TObjectPtr<const UClass> SourceClass;

	/// Human-readable description of the source.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FString DebugString;

	/// Path to the underlying asset, if any.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FSoftObjectPath SourceAssetPath;

	/// Availability of this source.
	UPROPERTY(VisibleAnywhere, Category = "DataRegistry")
	EDataRegistryAvailability Availability = EDataRegistryAvailability::DoesNotExist;

	/// Whether the source is initialized.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	bool bIsInitialized = false;

	/// Whether this is a runtime-only (transient) source.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	bool bIsTransient = false;

	/// Debug string of the parent (original) source, if this is a transient child. Empty otherwise.
	UPROPERTY(BlueprintReadWrite, Category = "DataRegistry")
	FString ParentSourceDebugString;
};

/// Provides tools for querying and inspecting Data Registries.
UCLASS(BlueprintType, Hidden)
class UDataRegistryTools : public UToolsetDefinition
{
	GENERATED_BODY()

public:
	/**
	 * Returns the names of all registered Data Registries.
	 * @param StructFilter If non-null, only registries whose item struct inherits from this
	 *   struct are returned.
	 * @return A list of registry names.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static TArray<FString> ListRegistries(const UScriptStruct* StructFilter = nullptr);

	/**
	 * Returns detailed information about a specific registry.
	 * @param RegistryName The registry name.
	 * @return Detailed info including description and id format.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static FDataRegistryInfo GetRegistryInfo(const FString& RegistryName);

	/**
	 * Returns the item struct schema as JSON.
	 * @param RegistryName The registry name.
	 * @return A JSON string describing the struct's fields and types.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static FString GetSchema(const FString& RegistryName);

	/**
	 * Returns all item names in a Data Registry.
	 * @param RegistryName The registry name.
	 * @return All item names defined on the registry, in registry order.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static TArray<FString> ListItems(const FString& RegistryName);

	/**
	 * Returns the editor-defined sources configured on a Data Registry.
	 * These are the sources as authored on the registry asset, before any runtime expansion
	 * of meta sources.
	 * @param RegistryName The registry name.
	 * @return A list of source summaries, in definition order.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static TArray<FDataRegistrySourceSummary> ListDataSources(const FString& RegistryName);

	/**
	 * Returns the runtime sources for a Data Registry.
	 * This is the expanded list including transient child sources generated from meta sources.
	 * Will equal ListDataSources when the registry has no meta sources.
	 * @param RegistryName The registry name.
	 * @return A list of source summaries, in runtime order.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static TArray<FDataRegistrySourceSummary> ListRuntimeSources(const FString& RegistryName);

	/**
	 * Returns cached item data.
	 * Items must be loaded in the registry cache to be returned.
	 * @param RegistryName The registry name.
	 * @param ItemNames Item names to retrieve.
	 * @return A map from item name to its struct data. Names that were not found in the cache
	 *   are simply omitted from the result.
	 */
	UFUNCTION(meta = (AICallable), Category = "DataRegistry")
	static TMap<FString, FInstancedStruct> GetItems(
		const FString& RegistryName, const TArray<FString>& ItemNames);
};
