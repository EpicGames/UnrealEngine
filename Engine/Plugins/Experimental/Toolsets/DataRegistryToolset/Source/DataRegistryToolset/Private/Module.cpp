// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTools.h"

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ToolsetRegistry/UToolsetRegistry.h"

class FDataRegistryToolsetModule : public IModuleInterface
{
public:
	virtual void StartupModule() override
	{
		UToolsetRegistry::RegisterToolsetClass(UDataRegistryTools::StaticClass());
	}

	virtual void ShutdownModule() override
	{
		if (UObjectInitialized())
		{
			UToolsetRegistry::UnregisterToolsetClass(UDataRegistryTools::StaticClass());
		}
	}
};

IMPLEMENT_MODULE(FDataRegistryToolsetModule, DataRegistryToolset);
