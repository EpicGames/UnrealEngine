// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataRegistryTools.h"

#include "Dom/JsonObject.h"
#include "Misc/AutomationTest.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonSerializer.h"
#include "StructUtils/InstancedStruct.h"
#include "Templates/UniquePtr.h"
#include "ToolsetRegistry/ToolCallExceptionHandler.h"

#if WITH_DEV_AUTOMATION_TESTS

BEGIN_DEFINE_SPEC(FDataRegistryToolsSpec, "AI.Toolsets.DataRegistryToolset",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

	TUniquePtr<UE::ToolsetRegistry::FToolCallExceptionHandler> ExceptionHandler;

	bool TryGetFirstRegistryName(FString& OutRegistryName);
	bool TryGetFirstRegistryNameWithItems(FString& OutRegistryName);
	TSharedPtr<FJsonObject> ParseJson(const FString& JsonString);
	void ExpectExceptionContains(const FString& Fragment);

END_DEFINE_SPEC(FDataRegistryToolsSpec)

bool FDataRegistryToolsSpec::TryGetFirstRegistryName(FString& OutRegistryName)
{
	TArray<FString> Registries = UDataRegistryTools::ListRegistries(nullptr);
	if (Registries.IsEmpty())
	{
		return false;
	}
	OutRegistryName = Registries[0];
	return true;
}

bool FDataRegistryToolsSpec::TryGetFirstRegistryNameWithItems(FString& OutRegistryName)
{
	TArray<FString> Registries = UDataRegistryTools::ListRegistries(nullptr);
	for (const FString& Name : Registries)
	{
		const FDataRegistryInfo Info = UDataRegistryTools::GetRegistryInfo(Name);
		if (Info.ItemCount > 0)
		{
			OutRegistryName = Name;
			return true;
		}
	}
	return false;
}

TSharedPtr<FJsonObject> FDataRegistryToolsSpec::ParseJson(const FString& JsonString)
{
	TSharedPtr<FJsonObject> Root;
	const TSharedRef<TJsonReader<>> Reader = TJsonReaderFactory<>::Create(JsonString);
	FJsonSerializer::Deserialize(Reader, Root);
	return Root;
}

void FDataRegistryToolsSpec::ExpectExceptionContains(const FString& Fragment)
{
	if (!TestTrue(TEXT("ExceptionHandler is valid"), ExceptionHandler.IsValid()))
	{
		return;
	}
	TestTrue(
		*FString::Printf(TEXT("Error contains '%s'"), *Fragment),
		ExceptionHandler->GetException().Contains(Fragment));
}

void FDataRegistryToolsSpec::Define()
{
	BeforeEach([this]()
	{
		ExceptionHandler = MakeUnique<UE::ToolsetRegistry::FToolCallExceptionHandler>();
	});

	AfterEach([this]()
	{
		ExceptionHandler.Reset();
	});

	Describe(TEXT("ListRegistries"), [this]()
	{
		It(TEXT("Returns an array"), [this]()
		{
			TArray<FString> Registries = UDataRegistryTools::ListRegistries(nullptr);
			TestTrue(TEXT("Is an array (possibly empty)"), Registries.Num() >= 0);
		});

		It(TEXT("Each entry is a non-empty name"), [this]()
		{
			TArray<FString> Registries = UDataRegistryTools::ListRegistries(nullptr);
			if (Registries.IsEmpty())
			{
				AddInfo(TEXT("Skipped: No data registries are loaded in this context."));
				return;
			}
			for (const FString& Name : Registries)
			{
				TestFalse(TEXT("name is not empty"), Name.IsEmpty());
			}
		});

		It(TEXT("Filter narrows results to matching item_struct"), [this]()
		{
			TArray<FString> AllNames = UDataRegistryTools::ListRegistries(nullptr);
			const UScriptStruct* SeedStruct = nullptr;
			for (const FString& Name : AllNames)
			{
				const FDataRegistryInfo Info = UDataRegistryTools::GetRegistryInfo(Name);
				if (Info.ItemStruct != nullptr)
				{
					SeedStruct = Info.ItemStruct.Get();
					break;
				}
			}
			if (SeedStruct == nullptr)
			{
				return;
			}

			TArray<FString> Filtered = UDataRegistryTools::ListRegistries(SeedStruct);
			TestTrue(TEXT("Filtered count is <= total"), Filtered.Num() <= AllNames.Num());
			for (const FString& Name : Filtered)
			{
				const FDataRegistryInfo Info = UDataRegistryTools::GetRegistryInfo(Name);
				if (TestNotNull(TEXT("Filtered entry has non-null item_struct"), Info.ItemStruct.Get()))
				{
					TestTrue(TEXT("Filtered item_struct matches filter"),
						Info.ItemStruct->IsChildOf(SeedStruct));
				}
			}
		});
	});

	Describe(TEXT("GetRegistryInfo"), [this]()
	{
		It(TEXT("Returns matching registry name"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			FDataRegistryInfo Info = UDataRegistryTools::GetRegistryInfo(RegName);
			TestEqual(TEXT("registry_name matches"), Info.RegistryName, RegName);
			TestTrue(TEXT("item_count >= 0"), Info.ItemCount >= 0);
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::GetRegistryInfo(TEXT("NonExistentRegistryType_XYZ"));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});

	Describe(TEXT("GetSchema"), [this]()
	{
		It(TEXT("Returns valid JSON"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			const FString Schema = UDataRegistryTools::GetSchema(RegName);
			TestFalse(TEXT("Schema not empty"), Schema.IsEmpty());
			TestNotNull(TEXT("Schema parses as JSON"), ParseJson(Schema).Get());
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::GetSchema(TEXT("NonExistentRegistryType_XYZ"));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});

	Describe(TEXT("ListItems"), [this]()
	{
		It(TEXT("Returns an array"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			TArray<FString> Items = UDataRegistryTools::ListItems(RegName);
			TestTrue(TEXT("Array is valid (possibly empty)"), Items.Num() >= 0);
		});

		It(TEXT("Returns one entry per registry item"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryNameWithItems(RegName))
			{
				return;
			}
			const FDataRegistryInfo Info = UDataRegistryTools::GetRegistryInfo(RegName);
			TArray<FString> Items = UDataRegistryTools::ListItems(RegName);
			TestEqual(TEXT("count matches item_count"), Items.Num(), Info.ItemCount);
			for (const FString& Name : Items)
			{
				TestFalse(TEXT("item name is non-empty"), Name.IsEmpty());
			}
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::ListItems(TEXT("NonExistentRegistryType_XYZ"));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});

	Describe(TEXT("GetItems"), [this]()
	{
		It(TEXT("Returns map keyed by item name"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryNameWithItems(RegName))
			{
				return;
			}
			TArray<FString> AllItems = UDataRegistryTools::ListItems(RegName);
			if (AllItems.IsEmpty())
			{
				return;
			}
			const TArray<FString> Probe({ AllItems[0] });
			const TMap<FString, FInstancedStruct> Items =
				UDataRegistryTools::GetItems(RegName, Probe);
			TestTrue(TEXT("Map contains requested item name"), Items.Contains(Probe[0]));
			if (const FInstancedStruct* Found = Items.Find(Probe[0]))
			{
				TestNotNull(TEXT("Item has a script struct"), Found->GetScriptStruct());
			}
		});

		It(TEXT("Missing items are omitted from result"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			const TMap<FString, FInstancedStruct> Items = UDataRegistryTools::GetItems(
				RegName, TArray<FString>({ TEXT("NonExistentItem_XYZ_12345") }));
			TestEqual(TEXT("Missing name is absent from result"), Items.Num(), 0);
		});

		It(TEXT("Empty names returns empty map"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			const TMap<FString, FInstancedStruct> Items =
				UDataRegistryTools::GetItems(RegName, TArray<FString>());
			TestEqual(TEXT("Map is empty"), Items.Num(), 0);
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::GetItems(
					TEXT("NonExistentRegistryType_XYZ"),
					TArray<FString>({ TEXT("SomeItem") }));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});

	Describe(TEXT("ListDataSources"), [this]()
	{
		It(TEXT("Returns an array"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			TArray<FDataRegistrySourceSummary> Sources =
				UDataRegistryTools::ListDataSources(RegName);
			TestTrue(TEXT("Array is valid (possibly empty)"), Sources.Num() >= 0);
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::ListDataSources(TEXT("NonExistentRegistryType_XYZ"));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});

	Describe(TEXT("ListRuntimeSources"), [this]()
	{
		It(TEXT("Returns an array"), [this]()
		{
			FString RegName;
			if (!TryGetFirstRegistryName(RegName))
			{
				return;
			}
			TArray<FDataRegistrySourceSummary> Sources =
				UDataRegistryTools::ListRuntimeSources(RegName);
			TestTrue(TEXT("Array is valid (possibly empty)"), Sources.Num() >= 0);
		});

		It(TEXT("Invalid registry raises script error"), [this]()
		{
			ExceptionHandler->CaptureErrorsIn([]()
			{
				UDataRegistryTools::ListRuntimeSources(TEXT("NonExistentRegistryType_XYZ"));
			});
			ExpectExceptionContains(TEXT("Registry 'NonExistentRegistryType_XYZ' not found"));
		});
	});
}

#endif // WITH_DEV_AUTOMATION_TESTS
